{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "GASToolsets",
	"Description": "Toolsets for the Gameplay Ability System",
	"Category": "Other",
	"EditorOnly": true,
	"CreatedBy": "Epic Games",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"NoRedist": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "GASToolsets",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "GameplayAbilities",
			"Enabled": true
		},
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}
