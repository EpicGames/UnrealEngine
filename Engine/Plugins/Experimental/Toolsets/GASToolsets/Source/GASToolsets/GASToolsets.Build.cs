// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GASToolsets : ModuleRules
{
	public GASToolsets(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.Add("Core");

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"AssetRegistry",
			"CoreUObject",
			"Engine",
			"GameplayAbilities",
			"GameplayTags",
			"GameplayTagsEditor",
			"Kismet",
			"KismetCompiler",
			"ToolsetRegistry",
			"UnrealEd",
		});
	}
}
