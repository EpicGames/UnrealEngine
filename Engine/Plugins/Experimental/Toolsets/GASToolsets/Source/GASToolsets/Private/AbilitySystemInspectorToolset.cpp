// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemInspectorToolset.h"

#include "AbilitySystemBlueprintLibrary.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "AttributeSet.h"
#include "Engine/World.h"
#include "GameplayEffect.h"
#include "GameplayEffectTypes.h"
#include "Kismet/KismetSystemLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilitySystemInspectorToolset)

namespace
{
	/**
	 * Returns the AbilitySystemComponent for the given actor.
	 * Raises a script error and returns nullptr if Actor is null or has no ASC.
	 */
	UAbilitySystemComponent* GetASCFromActor(AActor* Actor)
	{
		if (!Actor)
		{
			UKismetSystemLibrary::RaiseScriptError(TEXT("Actor is null."));
			return nullptr;
		}

		UAbilitySystemComponent* ASC =
			UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(Actor);

		if (!ASC)
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("Actor '%s' does not have an AbilitySystemComponent."),
				*Actor->GetActorLabel()));
			return nullptr;
		}

		return ASC;
	}
}

TArray<FRuntimeAttributeValue> UAbilitySystemInspectorToolset::GetAttributeValues(AActor* Actor)
{
	UAbilitySystemComponent* ASC = GetASCFromActor(Actor);
	if (!ASC)
	{
		return {};
	}

	AActor* OwnerActor = ASC->GetOwner();

	TArray<FRuntimeAttributeValue> Results;

	for (const UAttributeSet* Set : ASC->GetSpawnedAttributes())
	{
		if (!Set)
		{
			continue;
		}

		UClass* SetClass = Set->GetClass();
		const FString SetClassName = SetClass->GetName();

		TArray<FGameplayAttribute> Attributes;
		UAttributeSet::GetAttributesFromSetClass(SetClass, Attributes);

		for (const FGameplayAttribute& Attr : Attributes)
		{
			bool bFoundBase = false;
			bool bFoundCurrent = false;
			const float BaseVal =
				UAbilitySystemBlueprintLibrary::GetFloatAttributeBase(OwnerActor, Attr, bFoundBase);
			const float CurrentVal =
				UAbilitySystemBlueprintLibrary::GetFloatAttribute(OwnerActor, Attr, bFoundCurrent);

			FRuntimeAttributeValue Info;
			Info.AttributeName = Attr.GetName();
			Info.SetClassName = SetClassName;
			Info.FullName = FString::Printf(TEXT("%s.%s"), *SetClassName, *Info.AttributeName);
			Info.BaseValue = bFoundBase ? BaseVal : 0.f;
			Info.CurrentValue = bFoundCurrent ? CurrentVal : 0.f;
			Results.Add(Info);
		}
	}

	Results.Sort([](const FRuntimeAttributeValue& A, const FRuntimeAttributeValue& B)
	{
		return A.FullName < B.FullName;
	});

	return Results;
}

TArray<FActiveEffectInfo> UAbilitySystemInspectorToolset::GetActiveEffects(AActor* Actor)
{
	UAbilitySystemComponent* ASC = GetASCFromActor(Actor);
	if (!ASC)
	{
		return {};
	}

	TArray<FActiveEffectInfo> Results;

	const float WorldTime = ASC->GetWorld() ? ASC->GetWorld()->GetTimeSeconds() : 0.f;

	const FActiveGameplayEffectsContainer& Container = ASC->GetActiveGameplayEffects();
	for (const FActiveGameplayEffect& ActiveEffect : &Container)
	{
		if (ActiveEffect.IsPendingRemove)
		{
			continue;
		}

		const UGameplayEffect* GEDef = ActiveEffect.Spec.Def.Get();

		FActiveEffectInfo Info;
		Info.EffectName = GEDef ? GEDef->GetName() : TEXT("Unknown");
		Info.StackCount =
			UAbilitySystemBlueprintLibrary::GetActiveGameplayEffectStackCount(ActiveEffect.Handle);
		Info.TotalDuration = ActiveEffect.GetDuration();
		Info.RemainingDuration = ActiveEffect.GetTimeRemaining(WorldTime);

		// Collect the tags the GE definition grants to the owner.
		if (GEDef)
		{
			const FGameplayTagContainer& GrantedTags =
				UAbilitySystemBlueprintLibrary::GetGameplayEffectGrantedTags(GEDef->GetClass());
			for (const FGameplayTag& Tag : GrantedTags)
			{
				Info.GrantedTags.Add(Tag.ToString());
			}
		}

		Results.Add(Info);
	}

	Results.Sort([](const FActiveEffectInfo& A, const FActiveEffectInfo& B)
	{
		return A.EffectName < B.EffectName;
	});

	return Results;
}

TArray<FGrantedAbilityInfo> UAbilitySystemInspectorToolset::GetGrantedAbilities(AActor* Actor)
{
	UAbilitySystemComponent* ASC = GetASCFromActor(Actor);
	if (!ASC)
	{
		return {};
	}

	TArray<FGrantedAbilityInfo> Results;

	for (const FGameplayAbilitySpec& Spec : ASC->GetActivatableAbilities())
	{
		FGrantedAbilityInfo Info;
		Info.AbilityName = Spec.Ability ? Spec.Ability->GetClass()->GetName() : TEXT("Unknown");
		Info.Level = Spec.Level;
		Info.bIsActive = Spec.IsActive();
		Results.Add(Info);
	}

	Results.Sort([](const FGrantedAbilityInfo& A, const FGrantedAbilityInfo& B)
	{
		return A.AbilityName < B.AbilityName;
	});

	return Results;
}

TArray<FString> UAbilitySystemInspectorToolset::GetActiveTags(AActor* Actor)
{
	UAbilitySystemComponent* ASC = GetASCFromActor(Actor);
	if (!ASC)
	{
		return {};
	}

	TArray<FString> TagStrings;
	for (const FGameplayTag& Tag : ASC->GetOwnedGameplayTags())
	{
		TagStrings.Add(Tag.ToString());
	}

	TagStrings.Sort();
	return TagStrings;
}
