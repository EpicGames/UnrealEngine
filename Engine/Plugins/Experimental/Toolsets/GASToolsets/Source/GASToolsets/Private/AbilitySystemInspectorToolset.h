// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolsetRegistry/ToolsetDefinition.h"

#include "AbilitySystemInspectorToolset.generated.h"

/// The runtime base and current value of a single gameplay attribute.
USTRUCT(BlueprintType)
struct FRuntimeAttributeValue
{
	GENERATED_BODY()

	/// The short attribute name (e.g. "Health").
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	FString AttributeName;

	/// The fully-qualified name including the owning set (e.g. "UMyHealthSet.Health").
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	FString FullName;

	/// The name of the AttributeSet class that owns this attribute (e.g. "UMyHealthSet").
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	FString SetClassName;

	/// The permanent base value, unaffected by temporary modifiers.
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	float BaseValue = 0.f;

	/// The current value after all active modifiers are applied.
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	float CurrentValue = 0.f;
};

/// Summarizes a gameplay effect that is currently active on an AbilitySystemComponent.
USTRUCT(BlueprintType)
struct FActiveEffectInfo
{
	GENERATED_BODY()

	/// The name of the GameplayEffect class (e.g. "GE_Burning").
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	FString EffectName;

	/// Current stack count. 1 for non-stacking effects.
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	int32 StackCount = 1;

	/// Total duration in seconds. -1 means infinite.
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	float TotalDuration = 0.f;

	/// Remaining duration in seconds. -1 means infinite.
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	float RemainingDuration = 0.f;

	/// Tags granted to the owner by this effect.
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	TArray<FString> GrantedTags;
};

/// Summarizes an ability that has been granted to an AbilitySystemComponent.
USTRUCT(BlueprintType)
struct FGrantedAbilityInfo
{
	GENERATED_BODY()

	/// The name of the ability class (e.g. "GA_Sprint").
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	FString AbilityName;

	/// The level at which this ability was granted.
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	int32 Level = 1;

	/// True if the ability is currently executing (at least one active instance).
	UPROPERTY(BlueprintReadWrite, Category = "AbilitySystem")
	bool bIsActive = false;
};

/**
 * Provides tools for inspecting the runtime state of an AbilitySystemComponent.
 * Each function takes a direct actor pointer and raises a script error if the
 * actor is null or has no AbilitySystemComponent.
 */
UCLASS(BlueprintType, Hidden)
class UAbilitySystemInspectorToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/**
	 * Returns the current base and modified values of all gameplay attributes
	 * on the actor's AbilitySystemComponent.
	 * @param Actor  The target actor.
	 * @return A list of attribute values sorted by full name.
	 *   Raises a script error if Actor is null or has no ASC.
	 */
	UFUNCTION(meta = (AICallable), Category = "AbilitySystem")
	static TArray<FRuntimeAttributeValue> GetAttributeValues(AActor* Actor);

	/**
	 * Returns all gameplay effects currently active on the actor's
	 * AbilitySystemComponent.
	 * @param Actor  The target actor.
	 * @return A list of active effect summaries sorted by effect name.
	 *   Raises a script error if Actor is null or has no ASC.
	 */
	UFUNCTION(meta = (AICallable), Category = "AbilitySystem")
	static TArray<FActiveEffectInfo> GetActiveEffects(AActor* Actor);

	/**
	 * Returns all abilities granted to the actor's AbilitySystemComponent.
	 * @param Actor  The target actor.
	 * @return A list of granted ability summaries sorted by ability name.
	 *   Raises a script error if Actor is null or has no ASC.
	 */
	UFUNCTION(meta = (AICallable), Category = "AbilitySystem")
	static TArray<FGrantedAbilityInfo> GetGrantedAbilities(AActor* Actor);

	/**
	 * Returns the gameplay tags currently owned by the actor's
	 * AbilitySystemComponent (includes loose tags, effect-granted tags, etc.).
	 * @param Actor  The target actor.
	 * @return A sorted list of gameplay tag name strings.
	 *   Raises a script error if Actor is null or has no ASC.
	 */
	UFUNCTION(meta = (AICallable), Category = "AbilitySystem")
	static TArray<FString> GetActiveTags(AActor* Actor);

	friend class FAbilitySystemInspectorToolsetSpec;
};
