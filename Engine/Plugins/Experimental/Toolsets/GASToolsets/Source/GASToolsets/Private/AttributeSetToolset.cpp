// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttributeSetToolset.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AttributeSet.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttributeSetToolset)

namespace
{
	FAttributeSetClassInfo BuildClassInfo(UClass* Class)
	{
		FAttributeSetClassInfo Info;
		Info.ClassName = Class->GetName();

		// Populate AssetPath for Blueprint-generated classes.
		if (UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(Class))
		{
			if (UBlueprint* BP = Cast<UBlueprint>(BPGC->ClassGeneratedBy))
			{
				Info.AssetPath = BP->GetPathName();
			}
		}

		TArray<FGameplayAttribute> Attributes;
		UAttributeSet::GetAttributesFromSetClass(Class, Attributes);

		for (const FGameplayAttribute& Attr : Attributes)
		{
			FGameplayAttributeInfo AttrInfo;
			AttrInfo.AttributeName = Attr.GetName();
			AttrInfo.SetClassName = Info.ClassName;
			AttrInfo.FullName = FString::Printf(TEXT("%s.%s"), *Info.ClassName, *AttrInfo.AttributeName);
			Info.Attributes.Add(AttrInfo);
		}

		return Info;
	}
}

TArray<FAttributeSetClassInfo> UAttributeSetToolset::FindAttributeSetClasses()
{
	TArray<FAttributeSetClassInfo> Results;

	// Collect all loaded UClass objects that are strict subclasses of UAttributeSet.
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* Class = *ClassIt;
		if (!Class->IsChildOf(UAttributeSet::StaticClass()) ||
			Class == UAttributeSet::StaticClass() ||
			Class->HasAnyClassFlags(CLASS_Abstract))
		{
			continue;
		}

		Results.Add(BuildClassInfo(Class));
	}

	Results.Sort([](const FAttributeSetClassInfo& A, const FAttributeSetClassInfo& B)
	{
		return A.ClassName < B.ClassName;
	});

	return Results;
}

TArray<FGameplayAttributeInfo> UAttributeSetToolset::ListAttributes(const FString& ClassName)
{
	// Search loaded classes by name.
	UClass* FoundClass = nullptr;
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* Class = *ClassIt;
		if (Class->GetName() == ClassName && Class->IsChildOf(UAttributeSet::StaticClass()))
		{
			FoundClass = Class;
			break;
		}
	}

	if (!FoundClass)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("AttributeSet class '%s' not found. Use FindAttributeSetClasses to list available classes."),
			*ClassName));
		return {};
	}

	TArray<FGameplayAttribute> Attributes;
	UAttributeSet::GetAttributesFromSetClass(FoundClass, Attributes);

	TArray<FGameplayAttributeInfo> Results;
	Results.Reserve(Attributes.Num());
	for (const FGameplayAttribute& Attr : Attributes)
	{
		FGameplayAttributeInfo Info;
		Info.AttributeName = Attr.GetName();
		Info.SetClassName = ClassName;
		Info.FullName = FString::Printf(TEXT("%s.%s"), *ClassName, *Info.AttributeName);
		Results.Add(Info);
	}

	return Results;
}
