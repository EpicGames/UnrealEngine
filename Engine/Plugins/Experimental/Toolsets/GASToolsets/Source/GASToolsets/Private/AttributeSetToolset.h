// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolsetRegistry/ToolsetDefinition.h"

#include "AttributeSetToolset.generated.h"

/// Describes a gameplay attribute belonging to an AttributeSet class.
USTRUCT(BlueprintType)
struct FGameplayAttributeInfo
{
	GENERATED_BODY()

	/// The short attribute name (e.g. "Health").
	UPROPERTY(BlueprintReadWrite, Category = "AttributeSet")
	FString AttributeName;

	/// The fully-qualified attribute name including the set class (e.g. "UMyHealthSet.Health").
	UPROPERTY(BlueprintReadWrite, Category = "AttributeSet")
	FString FullName;

	/// The name of the AttributeSet class that owns this attribute (e.g. "UMyHealthSet").
	UPROPERTY(BlueprintReadWrite, Category = "AttributeSet")
	FString SetClassName;
};

/// Describes an AttributeSet subclass found in the project.
USTRUCT(BlueprintType)
struct FAttributeSetClassInfo
{
	GENERATED_BODY()

	/// The UClass name of the attribute set (e.g. "UMyHealthSet").
	UPROPERTY(BlueprintReadWrite, Category = "AttributeSet")
	FString ClassName;

	/// The object path to the Blueprint asset, or empty for native C++ classes.
	UPROPERTY(BlueprintReadWrite, Category = "AttributeSet")
	FString AssetPath;

	/// The gameplay attributes defined on this set.
	UPROPERTY(BlueprintReadWrite, Category = "AttributeSet")
	TArray<FGameplayAttributeInfo> Attributes;
};

/// Provides tools for discovering AttributeSet classes and their gameplay attributes.
UCLASS(BlueprintType, Hidden)
class UAttributeSetToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/**
	 * Returns all AttributeSet subclasses found in the project, including their attributes.
	 * Covers both native C++ subclasses and Blueprint subclasses discovered via the asset registry.
	 * @return A list of attribute set descriptors sorted by class name.
	 */
	UFUNCTION(meta = (AICallable), Category = "AttributeSets")
	static TArray<FAttributeSetClassInfo> FindAttributeSetClasses();

	/**
	 * Returns the gameplay attributes defined on a specific AttributeSet class.
	 * @param ClassName The UClass name to look up, e.g. "UMyHealthSet".
	 *   Raises a script error if the class is not found or is not an AttributeSet subclass.
	 * @return A list of attribute descriptors defined on the class.
	 */
	UFUNCTION(meta = (AICallable), Category = "AttributeSets")
	static TArray<FGameplayAttributeInfo> ListAttributes(const FString& ClassName);

	friend class FAttributeSetToolsetSpec;
};
