// Copyright Epic Games, Inc. All Rights Reserved.

#include "GASToolsets.h"

#include "AbilitySystemInspectorToolset.h"
#include "AttributeSetToolset.h"
#include "Editor.h"
#include "GameplayCueToolset.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

/*virtual*/ void FGASToolsetsModule::StartupModule()
{
	UToolsetRegistry::RegisterToolsetClass(UGameplayCueToolset::StaticClass());
	UToolsetRegistry::RegisterToolsetClass(UAttributeSetToolset::StaticClass());
	UToolsetRegistry::RegisterToolsetClass(UAbilitySystemInspectorToolset::StaticClass());
}

/*virtual*/ void FGASToolsetsModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		UToolsetRegistry::UnregisterToolsetClass(UAbilitySystemInspectorToolset::StaticClass());
		UToolsetRegistry::UnregisterToolsetClass(UAttributeSetToolset::StaticClass());
		UToolsetRegistry::UnregisterToolsetClass(UGameplayCueToolset::StaticClass());
	}
}

IMPLEMENT_MODULE(FGASToolsetsModule, GASToolsets);
DEFINE_LOG_CATEGORY(LogGASToolsets);
