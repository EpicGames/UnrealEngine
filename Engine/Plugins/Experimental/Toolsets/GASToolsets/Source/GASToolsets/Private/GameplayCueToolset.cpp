// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCueToolset.h"

#include "AbilitySystemGlobals.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "GameplayCueManager.h"
#include "GameplayCueNotify_Actor.h"
#include "GameplayCueNotify_Static.h"
#include "GameplayTagsEditorModule.h"
#include "GameplayTagsManager.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Modules/ModuleManager.h"
#include "Selection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayCueToolset)

namespace
{
	static const FString GameplayCueRootTag = TEXT("GameplayCue");
	static const FName GameplayCueNameProperty = FName("GameplayCueName");

	TArray<FString> CollectDescendantTagNames(const TSharedPtr<FGameplayTagNode>& ParentNode)
	{
		TArray<FString> TagNames;
		TArray<TSharedPtr<FGameplayTagNode>> Queue = ParentNode->GetChildTagNodes();
		while (Queue.Num() > 0)
		{
			TSharedPtr<FGameplayTagNode> Node = Queue.Pop(EAllowShrinking::No);
			if (Node.IsValid())
			{
				TagNames.Add(Node->GetCompleteTagName().ToString());
				Queue.Append(Node->GetChildTagNodes());
			}
		}
		return TagNames;
	}

	void CollectNotifyAssets(
		UClass* BaseClass, const FString& NotifyType,
		TArray<FGameplayCueNotifyInfo>& OutInfos)
	{
		IAssetRegistry& Registry =
			FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		FARFilter Filter;
		Filter.ClassPaths.Add(BaseClass->GetClassPathName());
		Filter.bRecursiveClasses = true;
		Filter.bRecursivePaths = true;

		TArray<FAssetData> AssetDataList;
		Registry.GetAssets(Filter, AssetDataList);

		for (const FAssetData& Data : AssetDataList)
		{
			FString CueTagValue;
			if (!Data.GetTagValue(GameplayCueNameProperty, CueTagValue) || CueTagValue.IsEmpty())
			{
				continue;
			}

			FGameplayCueNotifyInfo& Info = OutInfos.AddDefaulted_GetRef();
			Info.CueTag = CueTagValue;
			Info.AssetPath = Data.GetSoftObjectPath().ToString();
			Info.AssetName = Data.AssetName.ToString();
			Info.NotifyType = NotifyType;
		}
	}
}

TArray<FString> UGameplayCueToolset::ListCues(const FString& ParentTag)
{
	UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();

	const FString& RootName = ParentTag.IsEmpty() ? GameplayCueRootTag : ParentTag;
	TSharedPtr<FGameplayTagNode> RootNode = TagManager.FindTagNode(FName(*RootName));

	if (!RootNode.IsValid())
	{
		if (!ParentTag.IsEmpty())
		{
			UKismetSystemLibrary::RaiseScriptError(
				FString::Printf(TEXT("Gameplay tag '%s' does not exist."), *ParentTag));
		}
		return {};
	}

	TArray<FString> TagNames = CollectDescendantTagNames(RootNode);
	TagNames.Sort();
	return TagNames;
}

FGameplayCueInfo UGameplayCueToolset::GetCueInfo(const FString& CueTag)
{
	UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
	if (!TagManager.FindTagNode(FName(*CueTag)).IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Gameplay cue tag '%s' does not exist."), *CueTag));
		return FGameplayCueInfo();
	}

	FGameplayCueInfo Info;
	Info.Tag = CueTag;
	Info.NotifyType = TEXT("None");

	IAssetRegistry& Registry =
		FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	const auto TryFindNotify = [&Registry, &CueTag, &Info](UClass* BaseClass, const FString& Type) -> bool
	{
		FARFilter Filter;
		Filter.ClassPaths.Add(BaseClass->GetClassPathName());
		Filter.bRecursiveClasses = true;
		Filter.TagsAndValues.Add(GameplayCueNameProperty, CueTag);

		TArray<FAssetData> Results;
		Registry.GetAssets(Filter, Results);

		if (Results.Num() > 0)
		{
			Info.NotifyAssetPath = Results[0].GetSoftObjectPath().ToString();
			Info.NotifyType = Type;
			return true;
		}
		return false;
	};

	if (!TryFindNotify(UGameplayCueNotify_Static::StaticClass(), TEXT("Static")))
	{
		TryFindNotify(AGameplayCueNotify_Actor::StaticClass(), TEXT("Actor"));
	}

	return Info;
}

bool UGameplayCueToolset::ExecuteCueOnSelectedActor(
	const FString& CueTag, float NormalizedMagnitude, FVector Location, FVector Normal)
{
	if (!GEditor)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Editor not found."));
		return false;
	}

	AActor* TargetActor = nullptr;
	for (FSelectionIterator Iter(*GEditor->GetSelectedActors()); Iter; ++Iter)
	{
		if (AActor* Actor = Cast<AActor>(*Iter))
		{
			TargetActor = Actor;
			break;
		}
	}

	if (!TargetActor)
	{
		UKismetSystemLibrary::RaiseScriptError(
			TEXT("No actor selected. Select an actor in the editor before executing the cue."));
		return false;
	}

	const FGameplayTag Tag =
		UGameplayTagsManager::Get().RequestGameplayTag(FName(*CueTag), false);
	if (!Tag.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Gameplay cue tag '%s' does not exist."), *CueTag));
		return false;
	}

	FGameplayCueParameters Params;
	Params.NormalizedMagnitude = NormalizedMagnitude;
	Params.Location = Location;
	Params.Normal = Normal;

	UGameplayCueManager::ExecuteGameplayCue_NonReplicated(TargetActor, Tag, Params);
	return true;
}

TArray<FGameplayCueNotifyInfo> UGameplayCueToolset::FindCueNotifyAssets(const FString& ParentTag)
{
	TArray<FGameplayCueNotifyInfo> Infos;
	CollectNotifyAssets(UGameplayCueNotify_Static::StaticClass(), TEXT("Static"), Infos);
	CollectNotifyAssets(AGameplayCueNotify_Actor::StaticClass(), TEXT("Actor"), Infos);

	if (!ParentTag.IsEmpty())
	{
		UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
		const FGameplayTag FilterTag = TagManager.RequestGameplayTag(FName(*ParentTag), false);
		if (!FilterTag.IsValid())
		{
			UKismetSystemLibrary::RaiseScriptError(
				FString::Printf(TEXT("Parent tag '%s' does not exist."), *ParentTag));
			return {};
		}

		Infos.RemoveAll([&TagManager, FilterTag](const FGameplayCueNotifyInfo& Item)
		{
			const FGameplayTag ItemTag =
				TagManager.RequestGameplayTag(FName(*Item.CueTag), false);
			return !ItemTag.IsValid() || !ItemTag.MatchesTag(FilterTag);
		});
	}

	Infos.Sort([](const FGameplayCueNotifyInfo& A, const FGameplayCueNotifyInfo& B)
	{
		return A.CueTag < B.CueTag;
	});

	return Infos;
}

FString UGameplayCueToolset::CreateCueNotifyAsset(
	const FString& CueTag, const FString& PackagePath,
	const FString& AssetName, bool bIsActor)
{
	if (CueTag.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("CueTag cannot be empty."));
		return FString();
	}
	if (PackagePath.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PackagePath cannot be empty."));
		return FString();
	}
	if (AssetName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("AssetName cannot be empty."));
		return FString();
	}

	const FGameplayTag Tag =
		UGameplayTagsManager::Get().RequestGameplayTag(FName(*CueTag), false);
	if (!Tag.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Gameplay cue tag '%s' does not exist. Add it first with AddCueTag."), *CueTag));
		return FString();
	}

	const FString FullPackagePath = PackagePath / AssetName;
	UPackage* Package = CreatePackage(*FullPackagePath);
	if (!Package)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Failed to create package at '%s'."), *FullPackagePath));
		return FString();
	}

	UClass* NotifyClass = bIsActor
		? AGameplayCueNotify_Actor::StaticClass()
		: UGameplayCueNotify_Static::StaticClass();

	UBlueprint* NewBP = FKismetEditorUtilities::CreateBlueprint(
		NotifyClass, Package, FName(*AssetName),
		BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(),
		FName("GameplayCueToolset"));

	if (!NewBP || !NewBP->GeneratedClass || !NewBP->GeneratedClass->GetDefaultObject())
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Failed to create GameplayCueNotify Blueprint '%s'."), *AssetName));
		return FString();
	}

	if (!bIsActor)
	{
		UGameplayCueNotify_Static* CDO = Cast<UGameplayCueNotify_Static>(NewBP->GeneratedClass->GetDefaultObject());
		CDO->GameplayCueTag = Tag;
		CDO->GameplayCueName = Tag.GetTagName();
	}
	else
	{
		AGameplayCueNotify_Actor* CDO = Cast<AGameplayCueNotify_Actor>(NewBP->GeneratedClass->GetDefaultObject());
		CDO->GameplayCueTag = Tag;
		CDO->GameplayCueName = Tag.GetTagName();
	}

	NewBP->MarkPackageDirty();
	return NewBP->GetPathName();
}

bool UGameplayCueToolset::AddCueTag(const FString& CueTag, const FString& Comment)
{
	if (CueTag.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("CueTag cannot be empty."));
		return false;
	}

	if (!CueTag.StartsWith(GameplayCueRootTag + TEXT(".")))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Gameplay cue tags must begin with '%s.' (got '%s')."),
			*GameplayCueRootTag, *CueTag));
		return false;
	}

	IGameplayTagsEditorModule& EditorModule =
		FModuleManager::LoadModuleChecked<IGameplayTagsEditorModule>("GameplayTagsEditor");

	const bool bSuccess = EditorModule.AddNewGameplayTagToINI(CueTag, Comment, NAME_None);
	if (!bSuccess)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Failed to add gameplay cue tag '%s'."), *CueTag));
	}
	return bSuccess;
}

bool UGameplayCueToolset::RemoveCueTag(const FString& CueTag)
{
	UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
	TSharedPtr<FGameplayTagNode> TagNode = TagManager.FindTagNode(FName(*CueTag));

	if (!TagNode.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Gameplay cue tag '%s' does not exist."), *CueTag));
		return false;
	}

	IGameplayTagsEditorModule& EditorModule =
		FModuleManager::LoadModuleChecked<IGameplayTagsEditorModule>("GameplayTagsEditor");

	const bool bSuccess = EditorModule.DeleteTagFromINI(TagNode);
	if (!bSuccess)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Failed to remove gameplay cue tag '%s'."), *CueTag));
	}
	return bSuccess;
}

TArray<FString> UGameplayCueToolset::FindCueTagsWithoutNotifies()
{
	TArray<FString> AllCueTags = ListCues(FString());
	if (AllCueTags.IsEmpty())
	{
		return {};
	}

	TArray<FGameplayCueNotifyInfo> AllNotifies = FindCueNotifyAssets(FString());
	TSet<FString> TagsWithNotifies;
	for (const FGameplayCueNotifyInfo& Info : AllNotifies)
	{
		TagsWithNotifies.Add(Info.CueTag);
	}

	TArray<FString> Unimplemented;
	for (const FString& Tag : AllCueTags)
	{
		if (!TagsWithNotifies.Contains(Tag))
		{
			Unimplemented.Add(Tag);
		}
	}

	Unimplemented.Sort();
	return Unimplemented;
}
