// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolsetRegistry/ToolsetDefinition.h"

#include "GameplayCueToolset.generated.h"

/// Describes a registered gameplay cue and its associated notify asset.
USTRUCT(BlueprintType)
struct FGameplayCueInfo
{
	GENERATED_BODY()

	/// The fully-qualified gameplay cue tag (e.g. "GameplayCue.Character.Death").
	UPROPERTY(BlueprintReadWrite, Category = "GameplayCue")
	FString Tag;

	/// The object path to the associated GameplayCueNotify asset.
	/// Empty if no notify has been found for this tag in the asset registry.
	UPROPERTY(BlueprintReadWrite, Category = "GameplayCue")
	FString NotifyAssetPath;

	/// The notify base class: "Static", "Actor", or "None" if no notify is found.
	UPROPERTY(BlueprintReadWrite, Category = "GameplayCue")
	FString NotifyType;
};

/// Describes a GameplayCueNotify asset found in the project.
USTRUCT(BlueprintType)
struct FGameplayCueNotifyInfo
{
	GENERATED_BODY()

	/// The gameplay cue tag this notify responds to.
	UPROPERTY(BlueprintReadWrite, Category = "GameplayCue")
	FString CueTag;

	/// The full object path to the asset (e.g. "/Game/Effects/GCN_Death.GCN_Death").
	UPROPERTY(BlueprintReadWrite, Category = "GameplayCue")
	FString AssetPath;

	/// The asset name without the package path (e.g. "GCN_Death").
	UPROPERTY(BlueprintReadWrite, Category = "GameplayCue")
	FString AssetName;

	/// "Static" for UGameplayCueNotify_Static subclasses, "Actor" for AGameplayCueNotify_Actor subclasses.
	UPROPERTY(BlueprintReadWrite, Category = "GameplayCue")
	FString NotifyType;
};

/// Provides tools for inspecting, executing, and managing gameplay cues and their notify assets.
UCLASS(BlueprintType, Hidden)
class UGameplayCueToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/**
	 * Returns gameplay cue tags registered in the project.
	 * @param ParentTag If non-empty, only cues descending from this tag are returned.
	 *   For example, passing "GameplayCue.Character" returns all cues under that namespace.
	 *   Pass an empty string to return all tags under the "GameplayCue" root.
	 * @return A sorted list of fully-qualified gameplay cue tag names.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayCues")
	static TArray<FString> ListCues(const FString& ParentTag);

	/**
	 * Returns information about a specific gameplay cue, including its notify asset.
	 * @param CueTag The fully-qualified gameplay cue tag, e.g. "GameplayCue.Character.Death".
	 * @return The cue's notify asset path and type ("Static", "Actor", or "None").
	 *   Raises a script error if the tag does not exist.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayCues")
	static FGameplayCueInfo GetCueInfo(const FString& CueTag);

	/**
	 * Executes a gameplay cue non-replicated on the currently selected actor in the editor.
	 * Useful for previewing cue effects without network replication.
	 * Requires a PIE session or a configured GameplayCueManager to produce visible results.
	 * @param CueTag The fully-qualified tag of the cue to execute, e.g. "GameplayCue.Character.Death".
	 * @param NormalizedMagnitude A normalized (0.0-1.0) magnitude value passed to the cue.
	 * @param Location World-space location parameter passed to the cue.
	 * @param Normal World-space direction parameter passed to the cue.
	 * @return True if the cue was dispatched. Raises a script error if no actor is selected
	 *   or the tag does not exist.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayCues")
	static bool ExecuteCueOnSelectedActor(
		const FString& CueTag, float NormalizedMagnitude, FVector Location, FVector Normal);

	/**
	 * Returns all GameplayCueNotify assets found in the project via the asset registry.
	 * @param ParentTag If non-empty, only notifies whose cue tag descends from this tag are returned.
	 *   Pass an empty string to return all notify assets in the project.
	 * @return A list of notify descriptors, sorted by cue tag.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayCues")
	static TArray<FGameplayCueNotifyInfo> FindCueNotifyAssets(const FString& ParentTag);

	/**
	 * Creates a new GameplayCueNotify Blueprint asset at the specified content browser location.
	 * This should ONLY be called after getting explicit direction or permission from the user.
	 * @param CueTag The gameplay cue tag the notify should respond to.
	 *   The tag must already exist; create it first with AddCueTag if needed.
	 * @param PackagePath The content browser folder for the asset, e.g. "/Game/Effects/Cues".
	 * @param AssetName The file name for the new asset, e.g. "GCN_CharacterDeath".
	 * @param bIsActor If true, creates a GameplayCueNotify_Actor (spawned actor in the world).
	 *   If false, creates a GameplayCueNotify_Static (instant effect, no spawned actor).
	 * @return The object path of the created asset, or an empty string on failure.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayCues")
	static FString CreateCueNotifyAsset(
		const FString& CueTag, const FString& PackagePath,
		const FString& AssetName, bool bIsActor);

	/**
	 * Adds a new gameplay cue tag to the project.
	 * This should ONLY be called after getting explicit direction or permission from the user.
	 * @param CueTag The fully-qualified tag to add. Must begin with "GameplayCue."
	 *   (e.g. "GameplayCue.Character.Death").
	 * @param Comment An optional developer comment describing the cue's purpose.
	 * @return True if the tag was added successfully.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayCues")
	static bool AddCueTag(const FString& CueTag, const FString& Comment);

	/**
	 * Removes a gameplay cue tag from the project.
	 * This should ONLY be called after getting explicit direction or permission from the user.
	 * @param CueTag The fully-qualified cue tag to remove (e.g. "GameplayCue.Character.Death").
	 * @return True if the tag was removed successfully.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayCues")
	static bool RemoveCueTag(const FString& CueTag);

	/**
	 * Returns gameplay cue tags that have no corresponding GameplayCueNotify asset in the project.
	 * Tags without notifies produce no visible effect when triggered at runtime.
	 * @return A sorted list of gameplay cue tag names with no associated notify assets.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayCues")
	static TArray<FString> FindCueTagsWithoutNotifies();

private:
	friend class FGameplayCueToolsetSpec;
};
