// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbilitySystemInspectorToolsetTest.h"

#include "AbilitySystemInspectorToolset.h"
#include "AbilitySystemComponent.h"
#include "Editor.h"
#include "Engine/World.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FAbilitySystemInspectorToolsetSpec,
	"AI.Toolsets.GASToolsets.AbilitySystemInspectorToolset",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	AActor* TestActor = nullptr;
END_DEFINE_SPEC(FAbilitySystemInspectorToolsetSpec)

void FAbilitySystemInspectorToolsetSpec::Define()
{
	// -------------------------------------------------------------------------
	// Null actor: all tools must return empty.
	// -------------------------------------------------------------------------
	Describe(TEXT("With a null actor"), [this]()
	{
		It(TEXT("GetAttributeValues returns empty"), [this]()
		{
			TArray<FRuntimeAttributeValue> Values =
				UAbilitySystemInspectorToolset::GetAttributeValues(nullptr);
			TestEqual(TEXT("Empty"), Values.Num(), 0);
		});

		It(TEXT("GetActiveEffects returns empty"), [this]()
		{
			TArray<FActiveEffectInfo> Effects =
				UAbilitySystemInspectorToolset::GetActiveEffects(nullptr);
			TestEqual(TEXT("Empty"), Effects.Num(), 0);
		});

		It(TEXT("GetGrantedAbilities returns empty"), [this]()
		{
			TArray<FGrantedAbilityInfo> Abilities =
				UAbilitySystemInspectorToolset::GetGrantedAbilities(nullptr);
			TestEqual(TEXT("Empty"), Abilities.Num(), 0);
		});

		It(TEXT("GetActiveTags returns empty"), [this]()
		{
			TArray<FString> Tags = UAbilitySystemInspectorToolset::GetActiveTags(nullptr);
			TestEqual(TEXT("Empty"), Tags.Num(), 0);
		});
	});

	// -------------------------------------------------------------------------
	// Actor with no ASC: all tools must return empty.
	// -------------------------------------------------------------------------
	Describe(TEXT("With an actor that has no AbilitySystemComponent"), [this]()
	{
		BeforeEach([this]()
		{
			if (!GEditor) return;

			UWorld* World = GEditor->GetEditorWorldContext().World();
			if (!World) return;

			FActorSpawnParameters Params;
			Params.ObjectFlags = RF_Transient;
			TestActor = World->SpawnActor<AActor>(AActor::StaticClass(),
				FTransform::Identity, Params);
		});

		AfterEach([this]()
		{
			if (TestActor)
			{
				TestActor->Destroy();
				TestActor = nullptr;
			}
		});

		It(TEXT("GetAttributeValues returns empty"), [this]()
		{
			if (!TestNotNull(TEXT("TestActor"), TestActor)) return;
			TArray<FRuntimeAttributeValue> Values =
				UAbilitySystemInspectorToolset::GetAttributeValues(TestActor);
			TestEqual(TEXT("Empty"), Values.Num(), 0);
		});

		It(TEXT("GetActiveEffects returns empty"), [this]()
		{
			if (!TestNotNull(TEXT("TestActor"), TestActor)) return;
			TArray<FActiveEffectInfo> Effects =
				UAbilitySystemInspectorToolset::GetActiveEffects(TestActor);
			TestEqual(TEXT("Empty"), Effects.Num(), 0);
		});

		It(TEXT("GetGrantedAbilities returns empty"), [this]()
		{
			if (!TestNotNull(TEXT("TestActor"), TestActor)) return;
			TArray<FGrantedAbilityInfo> Abilities =
				UAbilitySystemInspectorToolset::GetGrantedAbilities(TestActor);
			TestEqual(TEXT("Empty"), Abilities.Num(), 0);
		});

		It(TEXT("GetActiveTags returns empty"), [this]()
		{
			if (!TestNotNull(TEXT("TestActor"), TestActor)) return;
			TArray<FString> Tags = UAbilitySystemInspectorToolset::GetActiveTags(TestActor);
			TestEqual(TEXT("Empty"), Tags.Num(), 0);
		});
	});

	// -------------------------------------------------------------------------
	// Actor with a live ASC and the test attribute set registered.
	// -------------------------------------------------------------------------
	Describe(TEXT("With an actor that has an AbilitySystemComponent"), [this]()
	{
		BeforeEach([this]()
		{
			if (!GEditor) return;

			UWorld* World = GEditor->GetEditorWorldContext().World();
			if (!World) return;

			FActorSpawnParameters Params;
			Params.ObjectFlags = RF_Transient;
			TestActor = World->SpawnActor<AGASToolsetsTestActor>(
				AGASToolsetsTestActor::StaticClass(), FTransform::Identity, Params);

			if (AGASToolsetsTestActor* GASActor = Cast<AGASToolsetsTestActor>(TestActor))
			{
				GASActor->AbilitySystemComponent->InitStats(
					UGASToolsetsTestAttributeSet::StaticClass(), nullptr);
			}
		});

		AfterEach([this]()
		{
			if (TestActor)
			{
				TestActor->Destroy();
				TestActor = nullptr;
			}
		});

		// --- GetAttributeValues --------------------------------------------------

		It(TEXT("GetAttributeValues returns one entry per attribute"), [this]()
		{
			if (!TestNotNull(TEXT("TestActor"), TestActor)) return;
			TArray<FRuntimeAttributeValue> Values =
				UAbilitySystemInspectorToolset::GetAttributeValues(TestActor);
			TestEqual(TEXT("Two attributes"), Values.Num(), 2);
		});

		It(TEXT("GetAttributeValues includes Health and MaxHealth"), [this]()
		{
			if (!TestNotNull(TEXT("TestActor"), TestActor)) return;
			TArray<FRuntimeAttributeValue> Values =
				UAbilitySystemInspectorToolset::GetAttributeValues(TestActor);

			const bool bHasHealth = Values.ContainsByPredicate(
				[](const FRuntimeAttributeValue& V)
				{
					return V.AttributeName == TEXT("Health");
				});
			const bool bHasMaxHealth = Values.ContainsByPredicate(
				[](const FRuntimeAttributeValue& V)
				{
					return V.AttributeName == TEXT("MaxHealth");
				});

			TestTrue(TEXT("Has Health"), bHasHealth);
			TestTrue(TEXT("Has MaxHealth"), bHasMaxHealth);
		});

		It(TEXT("GetAttributeValues populates SetClassName and FullName"), [this]()
		{
			if (!TestNotNull(TEXT("TestActor"), TestActor)) return;
			TArray<FRuntimeAttributeValue> Values =
				UAbilitySystemInspectorToolset::GetAttributeValues(TestActor);
			if (!TestTrue(TEXT("Has values"), Values.Num() > 0)) return;

			for (const FRuntimeAttributeValue& Value : Values)
			{
				TestEqual(TEXT("SetClassName"), Value.SetClassName,
					TEXT("GASToolsetsTestAttributeSet"));
				TestEqual(TEXT("FullName"),
					Value.FullName,
					FString::Printf(TEXT("GASToolsetsTestAttributeSet.%s"), *Value.AttributeName));
			}
		});

		It(TEXT("GetAttributeValues results are sorted by FullName"), [this]()
		{
			if (!TestNotNull(TEXT("TestActor"), TestActor)) return;
			TArray<FRuntimeAttributeValue> Values =
				UAbilitySystemInspectorToolset::GetAttributeValues(TestActor);
			for (int32 i = 1; i < Values.Num(); ++i)
			{
				TestTrue(TEXT("Sorted"), Values[i - 1].FullName <= Values[i].FullName);
			}
		});

		// --- Baseline empty results for other tools ------------------------------

		It(TEXT("GetActiveEffects returns empty with no active effects"), [this]()
		{
			if (!TestNotNull(TEXT("TestActor"), TestActor)) return;
			TArray<FActiveEffectInfo> Effects =
				UAbilitySystemInspectorToolset::GetActiveEffects(TestActor);
			TestEqual(TEXT("No effects"), Effects.Num(), 0);
		});

		It(TEXT("GetGrantedAbilities returns empty with no granted abilities"), [this]()
		{
			if (!TestNotNull(TEXT("TestActor"), TestActor)) return;
			TArray<FGrantedAbilityInfo> Abilities =
				UAbilitySystemInspectorToolset::GetGrantedAbilities(TestActor);
			TestEqual(TEXT("No abilities"), Abilities.Num(), 0);
		});

		It(TEXT("GetActiveTags returns empty with no tags"), [this]()
		{
			if (!TestNotNull(TEXT("TestActor"), TestActor)) return;
			TArray<FString> Tags = UAbilitySystemInspectorToolset::GetActiveTags(TestActor);
			TestEqual(TEXT("No tags"), Tags.Num(), 0);
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
