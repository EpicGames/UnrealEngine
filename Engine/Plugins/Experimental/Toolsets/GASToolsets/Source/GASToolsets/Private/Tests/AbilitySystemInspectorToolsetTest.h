// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemInterface.h"
#include "Tests/AttributeSetToolsetTest.h"

#include "AbilitySystemInspectorToolsetTest.generated.h"

/// Minimal actor used in automation tests.
/// Owns an AbilitySystemComponent and the test attribute set.
UCLASS(Hidden, MinimalAPI)
class AGASToolsetsTestActor : public AActor, public IAbilitySystemInterface
{
	GENERATED_BODY()

public:
	AGASToolsetsTestActor()
	{
		AbilitySystemComponent =
			CreateDefaultSubobject<UAbilitySystemComponent>(TEXT("AbilitySystemComponent"));
	}

	virtual UAbilitySystemComponent* GetAbilitySystemComponent() const override
	{
		return AbilitySystemComponent;
	}

	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> AbilitySystemComponent;
};
