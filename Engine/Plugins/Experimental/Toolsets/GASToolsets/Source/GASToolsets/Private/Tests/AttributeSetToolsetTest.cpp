// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttributeSetToolsetTest.h"

#include "AttributeSetToolset.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FAttributeSetToolsetSpec, "AI.Toolsets.GASToolsets.AttributeSetToolset",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FAttributeSetToolsetSpec)

void FAttributeSetToolsetSpec::Define()
{
	Describe(TEXT("FindAttributeSetClasses"), [this]()
	{
		It(TEXT("Returns at least one class"), [this]()
		{
			TArray<FAttributeSetClassInfo> Classes = UAttributeSetToolset::FindAttributeSetClasses();
			TestTrue(TEXT("At least one class found"), Classes.Num() > 0);
		});

		It(TEXT("Includes the test attribute set class"), [this]()
		{
			TArray<FAttributeSetClassInfo> Classes = UAttributeSetToolset::FindAttributeSetClasses();
			const FAttributeSetClassInfo* Found = Classes.FindByPredicate(
				[](const FAttributeSetClassInfo& Info)
				{
					return Info.ClassName == TEXT("GASToolsetsTestAttributeSet");
				});
			TestNotNull(TEXT("Test class present"), Found);
		});

		It(TEXT("Reports correct attribute count for the test class"), [this]()
		{
			TArray<FAttributeSetClassInfo> Classes = UAttributeSetToolset::FindAttributeSetClasses();
			const FAttributeSetClassInfo* Found = Classes.FindByPredicate(
				[](const FAttributeSetClassInfo& Info)
				{
					return Info.ClassName == TEXT("GASToolsetsTestAttributeSet");
				});
			if (!TestNotNull(TEXT("Test class present"), Found)) return;

			TestEqual(TEXT("Two attributes"), Found->Attributes.Num(), 2);
		});

		It(TEXT("Populates FullName and SetClassName on each attribute"), [this]()
		{
			TArray<FAttributeSetClassInfo> Classes = UAttributeSetToolset::FindAttributeSetClasses();
			const FAttributeSetClassInfo* Found = Classes.FindByPredicate(
				[](const FAttributeSetClassInfo& Info)
				{
					return Info.ClassName == TEXT("GASToolsetsTestAttributeSet");
				});
			if (!TestNotNull(TEXT("Test class present"), Found)) return;

			for (const FGameplayAttributeInfo& Attr : Found->Attributes)
			{
				TestFalse(TEXT("AttributeName is not empty"), Attr.AttributeName.IsEmpty());
				TestEqual(TEXT("SetClassName matches class"), Attr.SetClassName,
					TEXT("GASToolsetsTestAttributeSet"));
				TestTrue(TEXT("FullName starts with class name"),
					Attr.FullName.StartsWith(TEXT("GASToolsetsTestAttributeSet.")));
			}
		});

		It(TEXT("Results are sorted by class name"), [this]()
		{
			TArray<FAttributeSetClassInfo> Classes = UAttributeSetToolset::FindAttributeSetClasses();
			for (int32 i = 1; i < Classes.Num(); ++i)
			{
				TestTrue(TEXT("Classes are sorted"),
					Classes[i - 1].ClassName <= Classes[i].ClassName);
			}
		});

		It(TEXT("Does not include the abstract UAttributeSet base class"), [this]()
		{
			TArray<FAttributeSetClassInfo> Classes = UAttributeSetToolset::FindAttributeSetClasses();
			const bool bContainsBase = Classes.ContainsByPredicate(
				[](const FAttributeSetClassInfo& Info)
				{
					return Info.ClassName == TEXT("AttributeSet");
				});
			TestFalse(TEXT("Base class excluded"), bContainsBase);
		});
	});

	Describe(TEXT("ListAttributes"), [this]()
	{
		It(TEXT("Returns the correct number of attributes for a valid class"), [this]()
		{
			TArray<FGameplayAttributeInfo> Attrs =
				UAttributeSetToolset::ListAttributes(TEXT("GASToolsetsTestAttributeSet"));
			TestEqual(TEXT("Two attributes"), Attrs.Num(), 2);
		});

		It(TEXT("Includes Health and MaxHealth attributes"), [this]()
		{
			TArray<FGameplayAttributeInfo> Attrs =
				UAttributeSetToolset::ListAttributes(TEXT("GASToolsetsTestAttributeSet"));

			const bool bHasHealth = Attrs.ContainsByPredicate(
				[](const FGameplayAttributeInfo& Info)
				{
					return Info.AttributeName == TEXT("Health");
				});
			const bool bHasMaxHealth = Attrs.ContainsByPredicate(
				[](const FGameplayAttributeInfo& Info)
				{
					return Info.AttributeName == TEXT("MaxHealth");
				});

			TestTrue(TEXT("Has Health"), bHasHealth);
			TestTrue(TEXT("Has MaxHealth"), bHasMaxHealth);
		});

		It(TEXT("FullName is formatted as SetClass.AttributeName"), [this]()
		{
			TArray<FGameplayAttributeInfo> Attrs =
				UAttributeSetToolset::ListAttributes(TEXT("GASToolsetsTestAttributeSet"));
			if (!TestTrue(TEXT("Has attributes"), Attrs.Num() > 0)) return;

			for (const FGameplayAttributeInfo& Attr : Attrs)
			{
				const FString ExpectedFullName =
					FString::Printf(TEXT("GASToolsetsTestAttributeSet.%s"), *Attr.AttributeName);
				TestEqual(TEXT("FullName format"), Attr.FullName, ExpectedFullName);
				TestEqual(TEXT("SetClassName"), Attr.SetClassName,
					TEXT("GASToolsetsTestAttributeSet"));
			}
		});

	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
