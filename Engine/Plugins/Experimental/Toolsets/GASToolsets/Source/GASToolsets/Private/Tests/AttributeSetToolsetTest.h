// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AttributeSet.h"

#include "AttributeSetToolsetTest.generated.h"

/// Minimal AttributeSet subclass used exclusively in automation tests.
/// Defines two FGameplayAttributeData properties to verify attribute enumeration.
UCLASS(Hidden, MinimalAPI)
class UGASToolsetsTestAttributeSet : public UAttributeSet
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadOnly, Category = "Attribute")
	FGameplayAttributeData Health;

	UPROPERTY(BlueprintReadOnly, Category = "Attribute")
	FGameplayAttributeData MaxHealth;
};
