// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGASToolsets, Log, Log);

class FGASToolsetsModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
